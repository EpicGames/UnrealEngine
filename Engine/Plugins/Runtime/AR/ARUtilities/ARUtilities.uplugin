{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "AR Utilities",
	"Description": "Utility code and content for AR systems",
	"Category": "Augmented Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Modules": [
		{
			"Name": "ARUtilities",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"ProgramAllowList": ["LiveLinkHub"]
		}
	],
	"Plugins": [
        {
            "Name": "LiveLink",
            "Enabled": true
        }
    ]
}
