// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ARUtilities : ModuleRules
{
	public ARUtilities(ReadOnlyTargetRules Target) : base(Target)
	{		
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"LiveLinkAnimationCore",
		});
			
		
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"Engine",
			"Slate",
			"SlateCore",
			"AugmentedReality",
			"MRMesh",
		});
	}
}
