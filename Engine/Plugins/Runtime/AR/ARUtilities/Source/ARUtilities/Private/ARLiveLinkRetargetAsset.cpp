// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARLiveLinkRetargetAsset.h"
#include "Features/IModularFeatures.h"

#if WITH_EDITOR
void UARLiveLinkRetargetAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Initialize the default bone map for each platform
	if (!BoneMap.Num())
	{
		if (PropertyChangedEvent.Property)
		{
			const FName PropertyName(PropertyChangedEvent.Property->GetFName());
			if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, SourceType))
			{
				if (SourceType == EARLiveLinkSourceType::ARKitPoseTracking)
				{
					BoneMap =
					{
						{TEXT("hips_joint"), 					TEXT("pelvis")},
						{TEXT("spine_2_joint"), 				TEXT("spine_01")},
						{TEXT("spine_4_joint"), 				TEXT("spine_02")},
						{TEXT("spine_7_joint"), 				TEXT("spine_03")},
						{TEXT("left_shoulder_1_joint"), 		TEXT("clavicle_l")},
						{TEXT("left_arm_joint"), 				TEXT("upperarm_l")},
						{TEXT("left_forearm_joint"), 			TEXT("lowerarm_l")},
						{TEXT("left_hand_joint"), 				TEXT("hand_l")},
						{TEXT("left_handIndex_1_joint"),		TEXT("index_01_l")},
						{TEXT("left_handIndex_2_joint"),		TEXT("index_02_l")},
						{TEXT("left_handIndex_3_joint"),		TEXT("index_03_l")},
						{TEXT("left_handMid_1_joint"), 			TEXT("middle_01_l")},
						{TEXT("left_handMid_2_joint"), 			TEXT("middle_02_l")},
						{TEXT("left_handMid_3_joint"), 			TEXT("middle_03_l")},
						{TEXT("left_handPinky_1_joint"),		TEXT("pinky_01_l")},
						{TEXT("left_handPinky_2_joint"),		TEXT("pinky_02_l")},
						{TEXT("left_handPinky_3_joint"),		TEXT("pinky_03_l")},
						{TEXT("left_handRing_1_joint"), 		TEXT("ring_01_l")},
						{TEXT("left_handRing_2_joint"), 		TEXT("ring_02_l")},
						{TEXT("left_handRing_3_joint"), 		TEXT("ring_03_l")},
						{TEXT("left_handThumbStart_joint"),		TEXT("thumb_01_l")},
						{TEXT("left_handThumb_1_joint"),		TEXT("thumb_02_l")},
						{TEXT("left_handThumb_2_joint"), 		TEXT("thumb_03_l")},
						{TEXT("right_shoulder_1_joint"), 		TEXT("clavicle_r")},
						{TEXT("right_arm_joint"), 				TEXT("upperarm_r")},
						{TEXT("right_forearm_joint"), 			TEXT("lowerarm_r")},
						{TEXT("right_hand_joint"), 				TEXT("hand_r")},
						{TEXT("right_handIndex_1_joint"), 		TEXT("index_01_r")},
						{TEXT("right_handIndex_2_joint"), 		TEXT("index_02_r")},
						{TEXT("right_handIndex_3_joint"), 		TEXT("index_03_r")},
						{TEXT("right_handMid_1_joint"), 		TEXT("middle_01_r")},
						{TEXT("right_handMid_2_joint"), 		TEXT("middle_02_r")},
						{TEXT("right_handMid_3_joint"), 		TEXT("middle_03_r")},
						{TEXT("right_handPinky_1_joint"), 		TEXT("pinky_01_r")},
						{TEXT("right_handPinky_2_joint"), 		TEXT("pinky_02_r")},
						{TEXT("right_handPinky_3_joint"), 		TEXT("pinky_03_r")},
						{TEXT("right_handRing_1_joint"), 		TEXT("ring_01_r")},
						{TEXT("right_handRing_2_joint"), 		TEXT("ring_02_r")},
						{TEXT("right_handRing_3_joint"), 		TEXT("ring_03_r")},
						{TEXT("right_handThumbStart_joint"),	TEXT("thumb_01_r")},
						{TEXT("right_handThumb_1_joint"), 		TEXT("thumb_02_r")},
						{TEXT("right_handThumb_2_joint"), 		TEXT("thumb_03_r")},
						{TEXT("neck_1_joint"), 					TEXT("neck_01")},
						{TEXT("head_joint"), 					TEXT("head")},
						{TEXT("left_upLeg_joint"), 				TEXT("thigh_l")},
						{TEXT("left_leg_joint"), 				TEXT("calf_l")},
						{TEXT("left_foot_joint"), 				TEXT("foot_l")},
						{TEXT("left_toes_joint"), 				TEXT("ball_l")},
						{TEXT("right_upLeg_joint"), 			TEXT("thigh_r")},
						{TEXT("right_leg_joint"), 				TEXT("calf_r")},
						{TEXT("right_foot_joint"), 				TEXT("foot_r")},
						{TEXT("right_toes_joint"), 				TEXT("ball_r")}
					};
				}
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UARLiveLinkRetargetAsset::BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose)
{
	static const auto& FeatureName = IARLiveLinkRetargetingLogic::GetModularFeatureName();
	auto& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(FeatureName))
	{
		auto& Logic = ModularFeatures.GetModularFeature<IARLiveLinkRetargetingLogic>(FeatureName);
		Logic.BuildPoseFromAnimationData(*this, DeltaTime, InSkeletonData, InFrameData, OutPose);
	}
}

FName UARLiveLinkRetargetAsset::GetRemappedBoneName(FName BoneName) const
{
	if (auto Record = BoneMap.Find(BoneName))
	{
		return *Record;
	}
	return BoneName;
}
