// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARPassthroughManager.h"
#include "ARActor.h"
#include "PassthroughMaterialUpdateComponent.h"
#include "ARLifeCycleComponent.h"
#include "Async/TaskGraphInterfaces.h"
#include "MRMeshComponent.h"


AARPassthroughManager::AARPassthroughManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));
	PassthroughUpdateComponent = CreateDefaultSubobject<UPassthroughMaterialUpdateComponent>(TEXT("PassthroughUpdateComponent"));
	
	// This makes the component editable in the actor's details panel
	PassthroughUpdateComponent->CreationMethod = EComponentCreationMethod::Instance;
}

void AARPassthroughManager::BeginPlay()
{
	Super::BeginPlay();
	
	UARLifeCycleComponent::OnSpawnARActorDelegate.AddUObject(this, &ThisClass::OnSpawnARActor);
}

void AARPassthroughManager::OnSpawnARActor(AARActor* NewARActor, UARComponent* NewARComponent, FGuid NativeID)
{
#if !UE_SERVER
	if (!NewARComponent)
	{
		return;
	}
	
	auto bIsValidComponent = false;
	
	for (auto ComponentClass : ARComponentClasses)
	{
		if (ComponentClass && NewARComponent->GetClass()->IsChildOf(ComponentClass))
		{
			bIsValidComponent = true;
			break;
		}
	}
	
	if (bIsValidComponent)
	{
		NewARComponent->OnMRMeshCreated.AddUObject(this, &ThisClass::OnMRMeshCreated);
		NewARComponent->OnMRMeshDestroyed.AddUObject(this, &ThisClass::OnMRMeshDestroyed);
		if (auto MeshComponent = NewARComponent->GetMRMesh())
		{
			OnMRMeshCreated(MeshComponent);
		}
	}
#endif
}

void AARPassthroughManager::OnMRMeshCreated(UMRMeshComponent* MeshComponent)
{
	PassthroughUpdateComponent->AddAffectedComponent(MeshComponent);
}

void AARPassthroughManager::OnMRMeshDestroyed(UMRMeshComponent* MeshComponent)
{
	PassthroughUpdateComponent->RemoveAffectedComponent(MeshComponent);
}

UPassthroughMaterialUpdateComponent* AARPassthroughManager::GetPassthroughMaterialUpdateComponent() const
{
	return PassthroughUpdateComponent;
}
