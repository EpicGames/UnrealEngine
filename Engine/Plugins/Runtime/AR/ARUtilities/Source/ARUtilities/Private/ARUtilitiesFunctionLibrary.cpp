// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARUtilitiesFunctionLibrary.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/Texture.h"


FVector2D UARUtilitiesFunctionLibrary::GetUVOffset(const FVector2D& ViewSize, const FVector2D& TextureSize)
{
	const auto ViewAspectRatio = ViewSize.X / ViewSize.Y;
	const auto TextureAspectRatio = TextureSize.X / TextureSize.Y;
	FVector2D Offset(0, 0);
	
	if (ViewAspectRatio > TextureAspectRatio)
	{
		Offset.Y = (1.f - TextureAspectRatio / ViewAspectRatio) * 0.5f;
	}
	else
	{
		Offset.X = (1.f - ViewAspectRatio / TextureAspectRatio) * 0.5f;
	}
	return Offset;
}

void UARUtilitiesFunctionLibrary::GetPassthroughCameraUVs(TArray<FVector2D>& OutUVs, const FVector2D& UVOffset)
{
	OutUVs.SetNumUninitialized(4);
	OutUVs[0] = FVector2D(UVOffset.X, UVOffset.Y);
	OutUVs[1] = FVector2D(UVOffset.X, 1.0f - UVOffset.Y);
	OutUVs[2] = FVector2D(1.0f - UVOffset.X, UVOffset.Y);
	OutUVs[3] = FVector2D(1.0f - UVOffset.X, 1.0f - UVOffset.Y);
}

void UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(UMaterialInstanceDynamic* MaterialInstance, UTexture* CameraTexture, float ColorScale)
{
	if (MaterialInstance && CameraTexture)
	{
		static const FName ParamName(TEXT("CameraTexture"));
		static const FName ExternalParamName(TEXT("ExternalCameraTexture"));
		static const FName ColorScaleParamName(TEXT("ColorScale"));
		const auto bIsExternalTexture = CameraTexture->GetMaterialType() == EMaterialValueType::MCT_TextureExternal;
		MaterialInstance->SetTextureParameterValue(bIsExternalTexture ? ExternalParamName : ParamName, CameraTexture);
		MaterialInstance->SetScalarParameterValue(ColorScaleParamName, ColorScale);
	}
}

void UARUtilitiesFunctionLibrary::UpdateSceneDepthTexture(UMaterialInstanceDynamic* MaterialInstance, UTexture* SceneDepthTexture, float DepthToMeterScale)
{
	if (MaterialInstance && SceneDepthTexture)
	{
		static const FName SceneDepthTextureParamName(TEXT("SceneDepthTexture"));
		static const FName DepthToMeterScaleParamName(TEXT("DepthToMeterScale"));
		
		MaterialInstance->SetTextureParameterValue(SceneDepthTextureParamName, SceneDepthTexture);
		MaterialInstance->SetScalarParameterValue(DepthToMeterScaleParamName, DepthToMeterScale);
	}
}

void UARUtilitiesFunctionLibrary::UpdateWorldToMeterScale(UMaterialInstanceDynamic* MaterialInstance, float WorldToMeterScale)
{
	if (MaterialInstance)
	{
		static const FName WorldToMeterScaleParamName(TEXT("WorldToMeterScale"));
		MaterialInstance->SetScalarParameterValue(WorldToMeterScaleParamName, WorldToMeterScale);
	}
}
