// Copyright Epic Games, Inc. All Rights Reserved.

#include "PassthroughMaterialUpdateComponent.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ARBlueprintLibrary.h"
#include "ARUtilitiesFunctionLibrary.h"


UPassthroughMaterialUpdateComponent::UPassthroughMaterialUpdateComponent()
{
#if UE_SERVER
	PrimaryComponentTick.bCanEverTick = false;
	PrimaryComponentTick.bStartWithTickEnabled = false;
#else
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultPassthroughMaterialRef(TEXT("/ARUtilities/Materials/M_ScreenSpacePassthroughCamera.M_ScreenSpacePassthroughCamera"));
	PassthroughMaterial = DefaultPassthroughMaterialRef.Object;
	
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> DefaultPassthroughMaterialExternalTextureRef(TEXT("/ARUtilities/Materials/MI_ScreenSpacePassthroughCameraExternalTexture.MI_ScreenSpacePassthroughCameraExternalTexture"));
	PassthroughMaterialExternalTexture = DefaultPassthroughMaterialExternalTextureRef.Object;
#endif
}

void UPassthroughMaterialUpdateComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

#if !UE_SERVER
	if (auto Texture = UARBlueprintLibrary::GetARTexture(TextureType))
	{
		const auto bIsExternalTexture = Texture->GetMaterialType() == EMaterialValueType::MCT_TextureExternal;
		
		// Create material instance for the pending components based on if the camera texture is external
		for (auto Component : PendingComponents)
		{
			if (Component)
			{
				Component->CreateAndSetMaterialInstanceDynamicFromMaterial(0, bIsExternalTexture ? PassthroughMaterialExternalTexture : PassthroughMaterial);
				AffectedComponents.Add(Component);
			}
		}
		
		PendingComponents = {};
		
		// Update the camera texture for all the affected components
		for (auto Component : AffectedComponents)
		{
			if (Component)
			{
				if (auto MaterialInstance = Cast<UMaterialInstanceDynamic>(Component->GetMaterial(0)))
				{
					UARUtilitiesFunctionLibrary::UpdateCameraTextureParam(MaterialInstance, Texture);
					
					static const FName DebugColorParam(TEXT("DebugColor"));
					MaterialInstance->SetVectorParameterValue(DebugColorParam, PassthroughDebugColor);
				}
			}
		}
	}
#endif
}

void UPassthroughMaterialUpdateComponent::AddAffectedComponent(UPrimitiveComponent* InComponent)
{
#if !UE_SERVER
	if (!InComponent)
	{
		return;
	}
	
	// Add the component to the pending list, we won't know which material to use until a valid AR texture is obtained
	PendingComponents.Add(InComponent);
#endif
}

void UPassthroughMaterialUpdateComponent::RemoveAffectedComponent(UPrimitiveComponent* InComponent)
{
#if !UE_SERVER
	if (!InComponent)
	{
		return;
	}
	
	AffectedComponents.RemoveSingleSwap(InComponent);
	PendingComponents.RemoveSingleSwap(InComponent);
#endif
}

void UPassthroughMaterialUpdateComponent::SetPassthroughDebugColor(FLinearColor NewDebugColor)
{
	PassthroughDebugColor = NewDebugColor;
}
