// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkRetargetAsset.h"
#include "Features/IModularFeature.h"

#include "ARLiveLinkRetargetAsset.generated.h"


UENUM(BlueprintType)
enum class EARLiveLinkSourceType : uint8
{
	None,
	ARKitPoseTracking,
};


USTRUCT(BlueprintType)
struct ARUTILITIES_API FARKitPoseTrackingConfig
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "LiveLink|ARKit")
	FVector HumanForward = FVector(-1, 0, 0);
	
	UPROPERTY(EditAnywhere, Category = "LiveLink|ARKit")
	FVector MeshForward = FVector(0, 1, 0);
};


/** Platform agnostic live link retarget asset */
UCLASS(Blueprintable, Abstract)
class ARUTILITIES_API UARLiveLinkRetargetAsset : public ULiveLinkRetargetAsset
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	EARLiveLinkSourceType SourceType = EARLiveLinkSourceType::None;
	
	/** Mapping from AR platform bone name to UE4 skeleton bone name */
	UPROPERTY(EditAnywhere, Category = "LiveLink")
	TMap<FName, FName> BoneMap;
	
	/** Configuration when using ARKit pose tracking */
	UPROPERTY(EditAnywhere, Category = "LiveLink|ARKit", meta = (editcondition = "SourceType == EARLiveLinkSourceType::ARKitPoseTracking"))
	FARKitPoseTrackingConfig ARKitPoseTrackingConfig;

public:
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	virtual void BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose) override;
	
	FName GetRemappedBoneName(FName BoneName) const;
};


/** Interface that allows each platform to implement its own retarting logic */
class IARLiveLinkRetargetingLogic : public IModularFeature
{
public:
	static const FName& GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("ARLiveLinkRetargetingLogic"));
		return FeatureName;
	}
	
	virtual void BuildPoseFromAnimationData(const UARLiveLinkRetargetAsset& SourceAsset, float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData,
											const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose) = 0;
};
