// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GameFramework/Actor.h"
#include "ARComponent.h"

#include "ARPassthroughManager.generated.h"


class UPassthroughMaterialUpdateComponent;
class AARActor;
class UMRMeshComponent;

/**
 * A helper actor that collects the ARComponent in the scene and apply the passthrough material to them.
 */
UCLASS(BlueprintType, ClassGroup = "AR")
class ARUTILITIES_API AARPassthroughManager : public AActor
{
	GENERATED_UCLASS_BODY()
	
public:
	UFUNCTION(BlueprintPure, Category = "Passthrough")
	UPassthroughMaterialUpdateComponent* GetPassthroughMaterialUpdateComponent() const;
	
protected:
	virtual void BeginPlay() override;
	
	/** What kinds of AR components should be gathered */
	UPROPERTY(EditAnywhere, Category = "Passthrough")
	TArray<TSubclassOf<UARComponent>> ARComponentClasses = { UARMeshComponent::StaticClass() };
	
private:
	void OnSpawnARActor(AARActor* NewARActor, UARComponent* NewARComponent, FGuid NativeID);
	void OnMRMeshCreated(UMRMeshComponent* MeshComponent);
	void OnMRMeshDestroyed(UMRMeshComponent* MeshComponent);
		
private:
	UPROPERTY()
	TObjectPtr<UPassthroughMaterialUpdateComponent> PassthroughUpdateComponent;
};
