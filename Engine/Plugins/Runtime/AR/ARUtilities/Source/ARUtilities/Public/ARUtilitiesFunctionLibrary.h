// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "ARUtilitiesFunctionLibrary.generated.h"


class UMaterialInstanceDynamic;
class UTexture;

UCLASS()
class ARUTILITIES_API UARUtilitiesFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	/** Return the UV offset by trying to fit a specific texture size onto the view (cropping) */
	static FVector2D GetUVOffset(const FVector2D& ViewSize, const FVector2D& TextureSize);
	
	/** Fill out quad vertices in OutUVs using the specified UV offset */
	static void GetPassthroughCameraUVs(TArray<FVector2D>& OutUVs, const FVector2D& UVOffset);
	
	/**
	 * Update material texture parameter using pre-defined names:
	 * For regular texture: CameraTexture
	 * For external texture: ExternalCameraTexture
	 */
	UFUNCTION(BlueprintCallable, Category = "AR Utilities")
	static void UpdateCameraTextureParam(UMaterialInstanceDynamic* MaterialInstance, UTexture* CameraTexture, float ColorScale = 1.f);
	
	/**
	 * Update material texture parameter using pre-defined names:
	 * Scene depth texture: SceneDepthTexture
	 * Depth to meter scale: DepthToMeterScale
	 */
	UFUNCTION(BlueprintCallable, Category = "AR Utilities")
	static void UpdateSceneDepthTexture(UMaterialInstanceDynamic* MaterialInstance, UTexture* SceneDepthTexture, float DepthToMeterScale = 1.f);
	
	/**
	 * Update material texture parameter using pre-defined names:
	 * World to meter scale: WorldToMeterScale
	 */
	UFUNCTION(BlueprintCallable, Category = "AR Utilities")
	static void UpdateWorldToMeterScale(UMaterialInstanceDynamic* MaterialInstance, float WorldToMeterScale = 100.f);
};
