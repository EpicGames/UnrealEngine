// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "ARTextures.h"

#include "PassthroughMaterialUpdateComponent.generated.h"


class UMaterialInterface;
class UPrimitiveComponent;

/**
 * Helper component that automatically pick the correct passthrough material to use
 * and handles updating the camera texture in the tick.
 */
UCLASS(BlueprintType, ClassGroup = "AR", meta=(BlueprintSpawnableComponent))
class ARUTILITIES_API UPassthroughMaterialUpdateComponent : public UActorComponent
{
	GENERATED_BODY()
	
public:
	UPassthroughMaterialUpdateComponent();
	
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	
	/** Add a component to be affected by the passthrough rendering */
	UFUNCTION(BlueprintCallable, Category = "Passthrough")
	void AddAffectedComponent(UPrimitiveComponent* InComponent);
	
	/** Remove the component from passthrough rendering */
	UFUNCTION(BlueprintCallable, Category = "Passthrough")
	void RemoveAffectedComponent(UPrimitiveComponent* InComponent);
	
	/** Update the passthrough debug color */
	UFUNCTION(BlueprintCallable, Category = "Passthrough")
	void SetPassthroughDebugColor(FLinearColor NewDebugColor);
	
protected:
	/** Which AR texture to use as the camera texture */
	UPROPERTY(EditAnywhere, Category = "Passthrough")
	EARTextureType TextureType = EARTextureType::CameraImage;
	
	/** Which material to use for a regular camera texture */
	UPROPERTY(EditAnywhere, Category = "Passthrough")
	TObjectPtr<UMaterialInterface> PassthroughMaterial;
	
	/** Which material to use for an external camera texture */
	UPROPERTY(EditAnywhere, Category = "Passthrough")
	TObjectPtr<UMaterialInterface> PassthroughMaterialExternalTexture;
	
	/**
	 * The debug color used to modulate the passthrough material.
	 * This can be used to visualize the affected meshes.
	 */
	UPROPERTY(EditAnywhere, Category = "Passthrough")
	FLinearColor PassthroughDebugColor = FLinearColor::White;
	
private:
	UPROPERTY(Transient)
	TArray<TObjectPtr<UPrimitiveComponent>> AffectedComponents;
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<UPrimitiveComponent>> PendingComponents;
};
